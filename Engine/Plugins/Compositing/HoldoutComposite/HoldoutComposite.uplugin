{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "HoldoutComposite",
	"Description": "Deprecated plugin now redirected to CompositeCore",
	"Category": "Compositing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
	],
	"Plugins": [
		{
			"Name": "CompositeCore",
			"Enabled": true
		}
	]
}